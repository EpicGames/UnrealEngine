// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class Shaders : ModuleRules
	{
		public Shaders(ReadOnlyTargetRules Target) : base(Target)
		{
			// External so we don't build a DLL for this, it is header only.
			Type = ModuleType.External;
			CppCompileWarningSettings.ModuleIncludePathWarningLevel = WarningLevel.Off;

			PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Public"));
			PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Shared"));
		}
	}
}
